{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Audio Insights Runtime",
	"Description": "Enable sending profiling and debugging data via Blueprints to the Audio Insights plugin.",
	"Category": "Audio",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "AudioInsightsRuntime",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	]
}