// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AudioInsightsRuntime : ModuleRules
{
	public AudioInsightsRuntime(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange
		(
			new string[]
			{
				"Core",
			}
		);	
		
		PrivateDependencyModuleNames.AddRange
		(
			new string[]
			{
				"AudioMixerCore",
				"CoreUObject",
				"Engine",
				"SlateCore",
			}
		);
	}
}
