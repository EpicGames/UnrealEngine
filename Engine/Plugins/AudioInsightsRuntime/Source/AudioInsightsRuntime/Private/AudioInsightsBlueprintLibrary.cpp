// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioInsightsBlueprintLibrary.h"

#include "Audio/AudioTraceUtil.h"
#include "AudioDevice.h"
#include "AudioMixerTrace.h"
#include "Engine/Engine.h"
#include "Styling/SlateIconFinder.h"
#include "UObject/Object.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioInsightsBlueprintLibrary)

void UAudioInsightsBlueprintLibrary::LogAudioInsightsEvent(const UObject* WorldContextObject, const FString& EventName, const USoundBase* SoundAsset, const AActor* Actor)
{
#if UE_AUDIO_PROFILERTRACE_ENABLED
	if (GEngine == nullptr || !GEngine->UseSound())
	{
		return;
	}

	const TObjectPtr<UWorld> ThisWorld = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (ThisWorld == nullptr || !ThisWorld->bAllowAudioPlayback)
	{
		return;
	}

	if (const FAudioDeviceHandle AudioDevice = ThisWorld->GetAudioDevice())
	{
		const TObjectPtr<UObject> SoundObjectPtr = SoundAsset != nullptr ? FSoftObjectPath(SoundAsset->GetPathName()).ResolveObject() : nullptr;
		const FString AssetPathName = SoundObjectPtr != nullptr ? SoundObjectPtr->GetPathName() : FString();
		const FString SoundBaseClassName = SoundObjectPtr != nullptr ? Audio::Trace::Util::GetSoundBaseAssetName(SoundObjectPtr->GetClass()) : FString();

		const FString ActorLabel = Actor != nullptr ? Actor->GetActorNameOrLabel() : FString();
		FString ActorIconName;
#if WITH_EDITOR
		if (Actor != nullptr)
		{
			FName IconName = Actor->GetCustomIconName();
			if (IconName == NAME_None)
			{
				// Actor didn't specify an icon - fallback on the class icon
				IconName = FSlateIconFinder::FindIconForClass(Actor->GetClass()).GetStyleName();
			}

			ActorIconName = IconName.ToString();
		}
#endif // WITH_EDITOR

		Audio::Trace::EventLog::SendEvent(AudioDevice->DeviceID, EventName, INDEX_NONE, AssetPathName, ActorLabel, ActorIconName, SoundBaseClassName);
	}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED
}

void UAudioInsightsBlueprintLibrary::LogAudioInsightsEventForAudioComponent(const UObject* WorldContextObject, const FString& EventName, const UAudioComponent* AudioComponent)
{
#if UE_AUDIO_PROFILERTRACE_ENABLED
	if (GEngine == nullptr || !GEngine->UseSound() || AudioComponent == nullptr)
	{
		return;
	}

	const TObjectPtr<UWorld> ThisWorld = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (ThisWorld == nullptr || !ThisWorld->bAllowAudioPlayback)
	{
		return;
	}

	if (FAudioDeviceHandle AudioDevice = ThisWorld->GetAudioDevice())
	{
		const Audio::FDeviceId AudioDeviceID = AudioDevice->DeviceID;
		AudioDevice->SendCommandToActiveSounds(AudioComponent->GetAudioComponentID(), [AudioDeviceID, EventName](FActiveSound& ActiveSound)
			{
				FAudioDeviceManager* AudioDeviceManager = GEngine->GetAudioDeviceManager();

				if (AudioDeviceManager)
				{
					FAudioDeviceHandle AudioDevice = AudioDeviceManager->GetAudioDevice(AudioDeviceID);
					if (AudioDevice.IsValid())
					{
						Audio::Trace::EventLog::SendActiveSoundEvent(ActiveSound, EventName);
					}
				}
			});
	}
#endif // UE_AUDIO_PROFILERTRACE_ENABLED
}