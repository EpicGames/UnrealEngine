// Copyright Epic Games, Inc. All Rights Reserved.
#include "AudioInsightsRuntimeModule.h"

#include "Modules/ModuleManager.h"

namespace UE::Audio::Insights
{
	void FAudioInsightsRuntimeModule::StartupModule()
	{
	}

	void FAudioInsightsRuntimeModule::ShutdownModule()
	{
	}

} // namespace UE::Audio::Insights

IMPLEMENT_MODULE(UE::Audio::Insights::FAudioInsightsRuntimeModule, AudioInsightsRuntime)
