// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once 

#include "Modules/ModuleInterface.h"

namespace UE::Audio::Insights
{
	class FAudioInsightsRuntimeModule final : public IModuleInterface
	{
	public:
		FAudioInsightsRuntimeModule() = default;
		virtual ~FAudioInsightsRuntimeModule() = default;

		FAudioInsightsRuntimeModule(const FAudioInsightsRuntimeModule&) = delete;
		FAudioInsightsRuntimeModule& operator=(const FAudioInsightsRuntimeModule&) = delete;
		FAudioInsightsRuntimeModule(FAudioInsightsRuntimeModule&&) = delete;
		FAudioInsightsRuntimeModule& operator=(FAudioInsightsRuntimeModule&&) = delete;

		virtual void StartupModule() override;
		virtual void ShutdownModule() override;
	};
} // namespace UE::Audio::Insights
