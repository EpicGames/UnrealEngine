// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "AudioInsightsBlueprintLibrary.generated.h"

class AActor;
class UAudioComponent;
class UObject;
class USoundBase;

#define UE_API AUDIOINSIGHTSRUNTIME_API

/** Static class with useful gameplay utility functions that can be called from both Blueprint and C++ */
UCLASS(meta = (ScriptName = "AudioInsightsBlueprintLibrary"), MinimalAPI)
class UAudioInsightsBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/**
	 * Send an event message to the Audio Insights event log.
	 * The filter category for the event can be added in the Audio Insights editor preferences
	 * Requires the Audio Insights plug-in to be enabled.
	 * @param EventName - the name that will appear in the event log for this event.
	 * @param SoundAsset (optional) - the sound asset associated with this event.
	 * @param Actor (optional) - the actor associated with this event.
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|AudioInsights", meta = (WorldContext = "WorldContextObject", AdvancedDisplay = "2"))
	static UE_API void LogAudioInsightsEvent(const UObject* WorldContextObject, const FString& EventName, const USoundBase* SoundAsset = nullptr, const AActor* Actor = nullptr);

	/**
	 * Send an event message to the Audio Insights event log for a given AudioComponent.
	 * The filter category for the event can be added in the Audio Insights editor preferences
	 * Requires the Audio Insights plug-in to be enabled.
	 * @param EventName - the name that will appear in the event log for this event
	 * @param AudioComponent - the AudioComponent associated with this event
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|AudioInsights", meta = (WorldContext = "WorldContextObject"))
	static UE_API void LogAudioInsightsEventForAudioComponent(const UObject* WorldContextObject, const FString& EventName, const class UAudioComponent* AudioComponent);

};

#undef UE_API